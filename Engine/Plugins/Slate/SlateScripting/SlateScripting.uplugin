{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Slate Scripting",
	"Description": "Allows interacting with Slate through scripting",
	"Category": "Scripting",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "SlateScriptingCommands",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}
