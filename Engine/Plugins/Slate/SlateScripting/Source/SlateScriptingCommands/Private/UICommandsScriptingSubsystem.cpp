// Copyright Epic Games, Inc. All Rights Reserved.

#include "UICommandsScriptingSubsystem.h"

#include "Framework/Commands/InputBindingManager.h"
#include "Framework/Commands/UICommandList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UICommandsScriptingSubsystem)

DEFINE_LOG_CATEGORY(LogCommandsScripting)


bool FScriptingCommandInfo::Equals(const FScriptingCommandInfo& InCommandInfo, const bool bCheckInputChord) const
{
	return InCommandInfo.Name == Name && InCommandInfo.Set == Set && InCommandInfo.ContextName == ContextName &&
		(!bCheckInputChord || InCommandInfo.InputChord == InputChord);
}

TSharedPtr<FUICommandInfo> FScriptingCommand::MakeUICommandInfo() const
{
	FInputBindingManager& BindingManager = FInputBindingManager::Get();
	const FName CommandName = GetFullName();
	const TSharedPtr<FBindingContext> Context = BindingManager.GetContextByName(CommandInfo.ContextName);

	if (!Context.IsValid())
	{
		UE_LOGF(LogCommandsScripting, Error, "Context is not registered in the Input Binding Manager: %ls", *CommandInfo.ContextName.ToString())
		return nullptr;
	}
	if (BindingManager.FindCommandInContext(CommandInfo.ContextName, CommandName))
	{
		UE_LOGF(LogCommandsScripting, Error, "%ls: Command already registered in context %ls", *CommandName.ToString(), *CommandInfo.ContextName.ToString())
		return nullptr;
	}
	if (BindingManager.FindCommandInContext(CommandInfo.ContextName, CommandInfo.InputChord, false))
	{
		UE_LOGF(LogCommandsScripting, Error, "Input Chord already mapped in context: %ls", *CommandInfo.ContextName.ToString())
		return nullptr;
	}

	TSharedPtr<FUICommandInfo> NewCommand;

	FUICommandInfo::MakeCommandInfo(
		Context.ToSharedRef(),
		NewCommand,
		CommandName,
		CommandInfo.Label,
		CommandInfo.Description,
		FSlateIcon(),
		EUserInterfaceActionType::Button,
		CommandInfo.InputChord
	);

	return NewCommand;
}

bool FScriptingCommand::UnregisterUICommandInfo() const
{
	FInputBindingManager& BindingManager = FInputBindingManager::Get();
	const TSharedPtr<FBindingContext> Context = BindingManager.GetContextByName(CommandInfo.ContextName);
	const TSharedPtr<FUICommandInfo> Command = BindingManager.FindCommandInContext(CommandInfo.ContextName, GetFullName());

	if (Context.IsValid() && Command.IsValid())
	{
		FUICommandInfo::UnregisterCommandInfo(Context.ToSharedRef(), Command.ToSharedRef());
		return true;
	}
	return false;
}



bool FScriptingCommandsContext::MapCommand(const TSharedRef<FScriptingCommand> ScriptingCommand)
{
	const TSharedPtr<FUICommandInfo> Command = FInputBindingManager::Get().FindCommandInContext(
		ScriptingCommand->CommandInfo.ContextName, ScriptingCommand->GetFullName());

	if (Command.IsValid())
	{
		CleanupPointerArray(CommandLists);

		for (TWeakPtr<FUICommandList> CommandList : CommandLists)
		{
			CommandList.Pin()->MapAction(Command, ScriptingCommand->OnExecuteAction, ScriptingCommand->OnCanExecuteAction);
		}
	
		ScriptingCommands.AddUnique(ScriptingCommand);
		return true;
	}
	
	UE_LOGF(LogCommandsScripting, Error, "Could not map command: %ls. The command could not be found in the Input Binding Manager", *ScriptingCommand->GetFullName().ToString())
	return false;
}

bool FScriptingCommandsContext::UnmapCommand(const TSharedRef<FScriptingCommand> ScriptingCommand)
{
	const TSharedPtr<FUICommandInfo> Command = FInputBindingManager::Get().FindCommandInContext(
		ScriptingCommand->CommandInfo.ContextName, ScriptingCommand->GetFullName());

	if (Command.IsValid())
	{
		CleanupPointerArray(CommandLists);
	
		for (TWeakPtr<FUICommandList> CommandList : CommandLists)
		{
			CommandList.Pin()->UnmapAction(Command);
		}

		ScriptingCommands.Remove(ScriptingCommand);
		return true;
	}
	
	UE_LOGF(LogCommandsScripting, Error, "Could not unmap command: %ls. The command could not be found in the Input Binding Manager", *ScriptingCommand->GetFullName().ToString())
	return false;
}

bool FScriptingCommandsContext::RegisterCommandList(const TSharedRef<FUICommandList> CommandList)
{
	CleanupPointerArray(CommandLists);

	if (!CommandLists.Contains(CommandList))
	{
		CommandLists.Add(CommandList);
		MapAllCommands(CommandList);
		return true;
	}
	
	UE_LOGF(LogCommandsScripting, Warning, "Trying to register an already registered command list")
	return false;
}

bool FScriptingCommandsContext::UnregisterCommandList(const TSharedRef<FUICommandList> CommandList)
{
	CleanupPointerArray(CommandLists);
	
	if (CommandLists.Remove(CommandList) > 0)
	{
		UnmapAllCommands(CommandList);
		return true;
	}
	
	UE_LOGF(LogCommandsScripting, Warning, "Trying to unregister a non registered command list")
	return false;
}

void FScriptingCommandsContext::MapAllCommands(const TSharedRef<FUICommandList> CommandList)
{
	for (const TSharedPtr<FScriptingCommand>& ScriptingCommand : ScriptingCommands)
	{
		if (ScriptingCommand.IsValid())
		{
			const TSharedPtr<FUICommandInfo> CommandInfo = FInputBindingManager::Get().FindCommandInContext(
				ContextName, ScriptingCommand->GetFullName());

			if (CommandInfo.IsValid())
			{
				CommandList->MapAction(CommandInfo, ScriptingCommand->OnExecuteAction, ScriptingCommand->OnCanExecuteAction);
			}	
		}
	}
}

void FScriptingCommandsContext::UnmapAllCommands(const TSharedRef<FUICommandList> CommandList)
{
	for (const TSharedPtr<FScriptingCommand>& ScriptingCommand : ScriptingCommands)
	{
		if (ScriptingCommand.IsValid())
		{
			const TSharedPtr<FUICommandInfo> CommandInfo = FInputBindingManager::Get().FindCommandInContext(
				ContextName, ScriptingCommand->GetFullName());

			if (CommandInfo.IsValid())
			{
				CommandList->UnmapAction(CommandInfo);
			}
		}
	}
}


void UUICommandsScriptingSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	FInputBindingManager::Get().OnRegisterCommandList.AddUObject(this, &UUICommandsScriptingSubsystem::RegisterCommandListForContext);
	FInputBindingManager::Get().OnUnregisterCommandList.AddUObject(this, &UUICommandsScriptingSubsystem::UnregisterCommandListForContext);
}

void UUICommandsScriptingSubsystem::Deinitialize()
{
	UnregisterAllSets();
}

void UUICommandsScriptingSubsystem::RegisterCommandListForContext(const FName ContextName,
                                                          const TSharedRef<FUICommandList> CommandList)
{
	FScriptingCommandsContext& CommandsContext = CommandsInContext.FindOrAdd(ContextName, ContextName);
	CommandsContext.RegisterCommandList(CommandList);
}

void UUICommandsScriptingSubsystem::UnregisterCommandListForContext(const FName ContextName,
                                                            const TSharedRef<FUICommandList> CommandList)
{
	FScriptingCommandsContext *CommandsContext = CommandsInContext.Find(ContextName);

	if (CommandsContext)
	{
		CommandsContext->UnregisterCommandList(CommandList);
	}
}

bool UUICommandsScriptingSubsystem::UnregisterContext(const FName ContextName)
{
	FScriptingCommandsContext *CommandsContext = CommandsInContext.Find(ContextName);

	if (CommandsContext)
	{
		CleanupPointerArray(CommandsContext->CommandLists);

		for (const TWeakPtr<FUICommandList>& CommandList : CommandsContext->CommandLists)
		{
			CommandsContext->UnmapAllCommands(CommandList.Pin().ToSharedRef());
		}
		
		CommandsInContext.Remove(ContextName);
		return true;
	}
	
	UE_LOGF(LogCommandsScripting, Warning, "Trying to unregister a non registered context")
	return false;
}



bool UUICommandsScriptingSubsystem::RegisterCommand(const FScriptingCommandInfo CommandInfo,
                                            const FExecuteCommand OnExecuteCommand,
                                            const bool bOverrideExisting)
{
	// Registers the command with default CanExecute (just checking if command set and subsystem are enabled).
	const FCanExecuteAction OnCanExecuteAction = FCanExecuteAction::CreateUObject(this,
		&UUICommandsScriptingSubsystem::DefaultCanExecuteAction, CommandInfo.Set);

	return RegisterNewScriptingCommand(CommandInfo, OnExecuteCommand, OnCanExecuteAction, bOverrideExisting);
}

bool UUICommandsScriptingSubsystem::RegisterCommandChecked(const FScriptingCommandInfo CommandInfo,
                                                   const FExecuteCommand OnExecuteCommand,
                                                   const FCanExecuteCommand OnCanExecuteCommand,
                                                   const bool bOverrideExisting)
{
	// Registers the command with the given CanExecute delegate (still checking if command set and subsystem are enabled)
	const FCanExecuteAction OnCanExecuteAction = FCanExecuteAction::CreateUObject(this,
		&UUICommandsScriptingSubsystem::HandleCanExecuteAction, OnCanExecuteCommand, CommandInfo);

	return RegisterNewScriptingCommand(CommandInfo, OnExecuteCommand, OnCanExecuteAction, bOverrideExisting);
}

bool UUICommandsScriptingSubsystem::UnregisterCommand(FScriptingCommandInfo CommandInfo)
{
	if (FScriptingCommandSet* CommandSet = CommandSets.Find(CommandInfo.Set))
	{
		TSharedPtr<FScriptingCommand>* ScriptingCommand = CommandSet->ScriptingCommands.FindByPredicate([CommandInfo](
			const TSharedPtr<FScriptingCommand> Command) { return Command.IsValid() && Command->CommandInfo.Equals(CommandInfo);});
		if (ScriptingCommand && ScriptingCommand->IsValid())
		{
			UnregisterScriptingCommand(ScriptingCommand->ToSharedRef());
			CommandSet->ScriptingCommands.RemoveSingle(*ScriptingCommand);
			return true;
		}
		
		UE_LOGF(LogCommandsScripting, Error, "Command not registered: %ls", *CommandInfo.GetFullName().ToString())
		return false;
	}
	
	UE_LOGF(LogCommandsScripting, Error, "Command Set not registered: %ls", *CommandInfo.Set.ToString())
	return false;
}

bool UUICommandsScriptingSubsystem::UnregisterCommandSet(const FName SetName)
{
	if (const FScriptingCommandSet* CommandSet = CommandSets.Find(SetName))
	{
		for (const TSharedPtr<FScriptingCommand>& ScriptingCommand : CommandSet->ScriptingCommands)
		{
			if (ScriptingCommand.IsValid())
			{
				UnregisterScriptingCommand(ScriptingCommand.ToSharedRef());
			}
		}
		CommandSets.Remove(SetName);
		return true;
	}
	
	UE_LOGF(LogCommandsScripting, Error, "Command Set not registered: %ls", *SetName.ToString())
	return false;
}

void UUICommandsScriptingSubsystem::UnregisterAllSets()
{
	for (const TPair<FName, FScriptingCommandSet>& CommandSet : CommandSets)
	{
		for (const TSharedPtr<FScriptingCommand>& ScriptingCommand : CommandSet.Value.ScriptingCommands)
		{
			if (ScriptingCommand.IsValid())
			{
				UnregisterScriptingCommand(ScriptingCommand.ToSharedRef());
			}
		}
	}
	CommandSets.Reset();
}

TArray<FName> UUICommandsScriptingSubsystem::GetAvailableContexts() const
{
	TArray<FName> OutContexts;

	CommandsInContext.GetKeys(OutContexts);
	return OutContexts;
}

bool UUICommandsScriptingSubsystem::IsContextRegistered(const FName ContextName) const
{
	return CommandsInContext.Contains(ContextName);
}

int UUICommandsScriptingSubsystem::GetBindingCountForContext(const FName ContextName)
{
	if (IsContextRegistered(ContextName))
	{
		CleanupPointerArray(CommandsInContext[ContextName].CommandLists);
		return CommandsInContext[ContextName].CommandLists.Num();
	}
	
	UE_LOGF(LogCommandsScripting, Error, "Context not registered: %ls", *ContextName.ToString())
	return 0;
}

TArray<FScriptingCommandInfo> UUICommandsScriptingSubsystem::GetRegisteredCommands() const
{
	TArray<FScriptingCommandInfo> OutCommandsInfo;

	for (const TPair<FName, FScriptingCommandSet>& CommandSet : CommandSets)
	{
		for (const TSharedPtr<FScriptingCommand>& ScriptingCommand : CommandSet.Value.ScriptingCommands)
		{
			if (ScriptingCommand.IsValid())
			{
				OutCommandsInfo.Add(ScriptingCommand->CommandInfo);
			}
		}
	}
	return OutCommandsInfo;
}

bool UUICommandsScriptingSubsystem::IsCommandRegistered(const FScriptingCommandInfo CommandInfo, const bool bCheckInputChord) const
{
	if (CommandSets.Contains(CommandInfo.Set))
	{
		const TSharedPtr<FScriptingCommand>* ExistingCommand = CommandSets[CommandInfo.Set].ScriptingCommands.FindByPredicate(
			[CommandInfo, bCheckInputChord](const TSharedPtr<FScriptingCommand> ScriptingCommand)
		{
			return ScriptingCommand.IsValid() && ScriptingCommand->CommandInfo.Equals(CommandInfo, bCheckInputChord);
		});
		
		return ExistingCommand && ExistingCommand->IsValid();
	}
	return false;
}

auto UUICommandsScriptingSubsystem::IsInputChordMapped(const FName ContextName, const FInputChord InputChord) const -> bool
{
	if (!IsContextRegistered(ContextName))
	{
		UE_LOGF(LogCommandsScripting, Error, "Context not registered: %ls", *ContextName.ToString())
		return false;
	}
	
	return FInputBindingManager::Get().FindCommandInContext(ContextName, InputChord, false) != nullptr;
}


void UUICommandsScriptingSubsystem::SetCanSetExecuteCommands(const FName SetName, const bool bShouldExecuteCommands)
{
	if (FScriptingCommandSet* CommandSet = CommandSets.Find(SetName))
	{
		CommandSet->bCanExecuteCommands = bShouldExecuteCommands;
	}
	else
	{
		UE_LOGF(LogCommandsScripting, Error, "Command Set not registered: %ls", *SetName.ToString())
	}
}

bool UUICommandsScriptingSubsystem::CanSetExecuteCommands(const FName SetName) const
{
	if (const FScriptingCommandSet* CommandSet = CommandSets.Find(SetName))
	{
		return bCanExecuteCommands && CommandSet->bCanExecuteCommands;
	}
	
	UE_LOGF(LogCommandsScripting, Error, "Command Set not registered: %ls", *SetName.ToString())
	return false;
}

void UUICommandsScriptingSubsystem::SetCanExecuteCommands(const bool bShouldExecuteCommands)
{
	bCanExecuteCommands = bShouldExecuteCommands;
}

bool UUICommandsScriptingSubsystem::IsCommandSetRegistered(const FName SetName) const
{
	return CommandSets.Contains(SetName);
}

bool UUICommandsScriptingSubsystem::RegisterCommandSet(const FName SetName)
{
	if (IsCommandSetRegistered(SetName))
	{
		UE_LOGF(LogCommandsScripting, Warning, "Command Set already registered: %ls", *SetName.ToString())
		return false;
	}

	CommandSets.Add(SetName);
	return true;
}

bool UUICommandsScriptingSubsystem::CanExecuteCommands() const
{
	return bCanExecuteCommands;
}



bool UUICommandsScriptingSubsystem::RegisterNewScriptingCommand(const FScriptingCommandInfo CommandInfo,
                                                      const FExecuteCommand OnExecuteCommand,
                                                      const FCanExecuteAction OnCanExecuteAction,
                                                      const bool bOverrideExisting)
{
	if (!IsCommandSetRegistered(CommandInfo.Set))
	{
		UE_LOGF(LogCommandsScripting, Error, "Trying to register a command in a non registered Command Set: %ls", *CommandInfo.Set.ToString())
		return false;
	}
	
	// Bind the given OnExecute delegate to the non dynamic one expected internally
	const FExecuteAction OnExecuteAction = FExecuteAction::CreateStatic(
		&UUICommandsScriptingSubsystem::HandleExecuteAction, OnExecuteCommand, CommandInfo);

	// Create the command data to be cached by the subsystem
	const TSharedPtr<FScriptingCommand> ScriptingCommand = MakeShareable(new FScriptingCommand(CommandInfo, OnExecuteAction, OnCanExecuteAction));

	// Unregister any command previously registered with this name to overwrite it
	if (IsCommandRegistered(CommandInfo, false))
	{
		if (!bOverrideExisting)
		{
			UE_LOGF(LogCommandsScripting, Warning, "Previously registered command %ls won't be overriden", *CommandInfo.GetFullName().ToString())
			return false;
		}
		if (!UnregisterCommand(CommandInfo))
		{
			UE_LOGF(LogCommandsScripting, Error, "Could not override command: %ls", *CommandInfo.GetFullName().ToString())
			return false;
		}
	}
	
	// Register the command without checking if it has been bound to a command list (if it wasn't, it will when possible)
	if (RegisterScriptingCommand(ScriptingCommand.ToSharedRef()))
	{
		// Add the command to our list of registered commands
		CommandSets[CommandInfo.Set].ScriptingCommands.AddUnique(ScriptingCommand);
		return true;
	}
	return false;
}

bool UUICommandsScriptingSubsystem::RegisterScriptingCommand(const TSharedRef<FScriptingCommand> ScriptingCommand)
{
	FScriptingCommandsContext* CommandsContext = CommandsInContext.Find(ScriptingCommand->CommandInfo.ContextName);

	if (CommandsContext)
	{
		return ScriptingCommand->MakeUICommandInfo() && CommandsContext->MapCommand(ScriptingCommand);
	}

	UE_LOGF(LogCommandsScripting, Error, "Context not registered: %ls", *ScriptingCommand->CommandInfo.ContextName.ToString())
	return false;
}

bool UUICommandsScriptingSubsystem::UnregisterScriptingCommand(const TSharedRef<FScriptingCommand> ScriptingCommand)
{
	FScriptingCommandsContext* CommandsContext = CommandsInContext.Find(ScriptingCommand->CommandInfo.ContextName);

	if (CommandsContext)
	{
		return CommandsContext->UnmapCommand(ScriptingCommand) && ScriptingCommand->UnregisterUICommandInfo();
	}

	UE_LOGF(LogCommandsScripting, Error, "Context not registered: %ls", *ScriptingCommand->CommandInfo.ContextName.ToString())
	return false;
}


void UUICommandsScriptingSubsystem::HandleExecuteAction(const FExecuteCommand OnExecuteAction, const FScriptingCommandInfo CommandInfo)
{
	OnExecuteAction.Execute(CommandInfo);
}

bool UUICommandsScriptingSubsystem::HandleCanExecuteAction(const FCanExecuteCommand OnCanExecuteAction, const FScriptingCommandInfo CommandInfo) const
{
	return CanSetExecuteCommands(CommandInfo.Set) && OnCanExecuteAction.Execute(CommandInfo);
}

bool UUICommandsScriptingSubsystem::DefaultCanExecuteAction(const FName SetName) const
{
	return CanSetExecuteCommands(SetName);
}
