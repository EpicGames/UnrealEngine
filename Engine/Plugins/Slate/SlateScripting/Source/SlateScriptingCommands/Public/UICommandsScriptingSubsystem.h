// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Subsystems/EngineSubsystem.h"

#include "Framework/Commands/UICommandList.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/InputChord.h"

#include "UICommandsScriptingSubsystem.generated.h"

#define UE_API SLATESCRIPTINGCOMMANDS_API


DECLARE_LOG_CATEGORY_EXTERN(LogCommandsScripting, Log, All);


/**
 * The data defining a scripting command. At the exception of its delegates.
 */
USTRUCT(BlueprintType)
struct FScriptingCommandInfo
{
	GENERATED_BODY()

	/** The editor context this command is bound to */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Editor Scripting | Commands")
	FName ContextName;

	/** The command set this command belongs to. This is to avoid conflicts and could refer to the owner of the command */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Editor Scripting | Commands")
	FName Set;

	/** The command name. Must be unique in its set. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Editor Scripting | Commands")
	FName Name;

	/** The command label or what name will be displayed for it */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Editor Scripting | Commands")
	FText Label;

	/** The description of the command */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Editor Scripting | Commands")
	FText Description;

	/** The input chord to bound to the command */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Editor Scripting | Commands")
	FInputChord InputChord;

	/** Compares this command info with the given one per context, set, name and optionally input chord */
	bool Equals(const FScriptingCommandInfo& InCommandInfo, const bool bCheckInputChord = false) const;

	/** Builds a full name in the format context.set.command_name to avoid conflicts with commands registered in different contexts/sets */
	FName GetFullName() const
	{
		// We append the set name and command name to avoid conflicts between commands registered by different scripts/components
		return *(ContextName.ToString() + "." + Set.ToString() + "." + Name.ToString());
	}
};

/** Exposing FExecuteAction as dynamic */
DECLARE_DYNAMIC_DELEGATE_OneParam(FExecuteCommand, FScriptingCommandInfo, CommandInfo);

/** Exposing FCanExecuteAction as dynamic */
DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(bool, FCanExecuteCommand, FScriptingCommandInfo, CommandInfo);

/**
 * All the internal data related to a scripting command as well as methods to easily expose the command to the binding manager
 */
USTRUCT()
struct FScriptingCommand
{
	GENERATED_BODY()

	FScriptingCommand() = default;
	
	FScriptingCommand(const FScriptingCommandInfo& InCommandInfo, const FExecuteAction InOnExecuteAction,
	                const FCanExecuteAction InOnCanExecuteAction) : CommandInfo(InCommandInfo),
																	OnExecuteAction(InOnExecuteAction),
																	OnCanExecuteAction(InOnCanExecuteAction)
	{}

	/** The command definition */
	FScriptingCommandInfo CommandInfo;

	/** The delegate to call when executing the command */
	FExecuteAction OnExecuteAction;

	/** The delegate to call to check whether the command can be executed */
	FCanExecuteAction OnCanExecuteAction;

	inline bool operator==(const FScriptingCommand& Other) const { return Other.CommandInfo.Name == CommandInfo.Name; }
	inline bool operator!=(const FScriptingCommand& Other) const { return !(*this == Other); }

	inline bool operator==(const FName OtherName) const { return OtherName == CommandInfo.Name; }
	inline bool operator!=(const FName OtherName) const { return !(*this == OtherName); }
	
	/** Builds a full name in the format context.set.command_name to avoid conflicts with commands registered in different contexts/sets  */
	FName GetFullName() const
	{
		return CommandInfo.GetFullName();
	}

	/**
	 * Registers a new UICommandInfo in the associated context through the binding manager.
	 * At this point, the command is exposed to the manager but not mapped to any command list.
	 */
	TSharedPtr<FUICommandInfo> MakeUICommandInfo() const;

	/** Unregisters this command's UICommandInfo from its associated context through the binding manager */
	bool UnregisterUICommandInfo() const;
};

/**
 * The list of commands and UI Command Lists associated with a context.
 * This enables easier management of commands within registered contexts and their UI Command Lists.
 */
USTRUCT()
struct FScriptingCommandsContext
{
	GENERATED_BODY()

	FScriptingCommandsContext() = default;

	FScriptingCommandsContext(const FName InContextName) : ContextName(InContextName) {}

	/** The context these command lists are bound to */
	FName ContextName;
	
	/** An array of command list associated to the given context */
	TArray<TWeakPtr<FUICommandList>> CommandLists;

	/** The commands bound in these command lists */
	TArray<TSharedPtr<FScriptingCommand>> ScriptingCommands;

public:
	/** Maps the given command to all the command lists of this context */
	bool MapCommand(const TSharedRef<FScriptingCommand> ScriptingCommand);

	/** Unmaps the given command from all the command lists of this context */
	bool UnmapCommand(const TSharedRef<FScriptingCommand> ScriptingCommand);

	/** Registers a command list in this context then map to it all commands existing within this context */
	bool RegisterCommandList(const TSharedRef<FUICommandList> CommandList);

	/** Unregisters command list from this context then unmap from it all commands existing within this context */
	bool UnregisterCommandList(const TSharedRef<FUICommandList> CommandList);

	/** Maps all commands existing within this context to the given command list */
	void MapAllCommands(TSharedRef<FUICommandList> CommandList);

	/** Unmaps all commands existing within this context from the given command list */
	void UnmapAllCommands(TSharedRef<FUICommandList> CommandList);
};


/**
 * To avoid conflicts between scripts, each command is associated to a command set which must be registered manually.
 */
USTRUCT()
struct FScriptingCommandSet
{
	GENERATED_BODY()

	/** The commands in this set */
	TArray<TSharedPtr<FScriptingCommand>> ScriptingCommands;
	
	/** Whether the commands in this set are enabled */
	bool bCanExecuteCommands = true;
};

/**
 * UEditorInputSubsystem
 * Subsystem for dynamically registering editor commands through scripting
 */
UCLASS(MinimalAPI)
class UUICommandsScriptingSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	/**
	 * USubsystem Interface: handles initialization of instances of the system
	 * This is where the subsystem binds to the CommandListRegistered delegate of the InputBindingManager
	 *
	 * To expose command lists from external systems to this subsystem, call FInputBindingManager::RegisterCommandList
	 * from the external system. The binding manager will then broadcast the registered command list to all subscribers.
	 */
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	
	/** USubsystem Interface: handles deinitialization of instances of the system */
	UE_API virtual void Deinitialize() override;

	/**
	 * Registers a command list in the given context. Optionally registering the context itself if it does not exist
	 * Note the context should already be existing and registered through the binding manager.
	 * 
	 * This should be called by modules and tools looking to expose their command lists to the subsystem.
	 * 
	 * @param	ContextName		The context name as defined in the binding manager
	 * @param	CommandList		The command list to register in this context
	 */
	UE_API void RegisterCommandListForContext(const FName ContextName, TSharedRef<FUICommandList> CommandList);

	/**
	 * Unregisters a command list from the given context.
	 * @param	ContextName		The context name as defined in the binding manager
	 * @param	CommandList		The command list to register in this context
	 */
	UE_API void UnregisterCommandListForContext(const FName ContextName, TSharedRef<FUICommandList> CommandList);

	/**
	 * Unregisters all command lists from the given context.
	 * @param	ContextName		The context name as defined in the binding manager
	 */
	UE_API bool UnregisterContext(const FName ContextName);

public:	
	/**
	 * Registers a command within the given context and set.
	 * The set must be registered beforehand.
	 * @param	CommandInfo			The command infos such as name, label, description and input chord.
	 * @param	OnExecuteCommand	The delegate to be executed for handling this command.
	 * @param	bOverrideExisting	Whether existing command with matching context, set and name should be overriden
	 * @return Whether the command was succesfully registered
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands |")
	UE_API bool RegisterCommand(FScriptingCommandInfo CommandInfo, FExecuteCommand OnExecuteCommand, bool bOverrideExisting = false);

	/**
	 * Registers a command within the given context and set.
	 * The set must be registered beforehand.
	 * @param	CommandInfo				The command infos such as name, label, description and input chord.
	 * @param	OnExecuteCommand 		The delegate to be executed for handling this command.
	 * @param	OnCanExecuteCommand 	The delegate to be executed for checking if this command can be executed.
	 * @param	bOverrideExisting		Whether existing command with matching context, set and name should be overriden
	 * @return Whether the command was successfully registered
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands |")
	UE_API bool RegisterCommandChecked(FScriptingCommandInfo CommandInfo, FExecuteCommand OnExecuteCommand,
	                            FCanExecuteCommand OnCanExecuteCommand, bool bOverrideExisting = false);

	/**
	 * Unregisters a command previously registered. The command name, set and context will be used for comparison.
	 * @return Whether the command was successfully unregistered.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands |")
	UE_API bool UnregisterCommand(FScriptingCommandInfo CommandInfo);

	/** Checks whether the given command is registered within the subsystem. Using name, set and context for comparison */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands |")
	UE_API bool IsCommandRegistered(FScriptingCommandInfo CommandInfo, bool bCheckInputChord = true) const;
	
	/** Retrieves the list of command info for all commands currently registered in the subsystem */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands | Global")
	UE_API TArray<FScriptingCommandInfo> GetRegisteredCommands() const;

	/** Checks whether commands registered in the subsystem can be executed */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands | Global")
	UE_API bool CanExecuteCommands() const;

	/** Sets whether commands registered in the subsystem can be executed */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands | Global")
	UE_API void SetCanExecuteCommands(bool bShouldExecuteCommands);

	/**
	 * Unregisters all commands dynamically registered within all contexts and sets.
	 * @warning this will unregister all commands currently registered by this subsystem
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands | Global")
	UE_API void UnregisterAllSets();

	
	/**
	 * Registers a new command set
	 * @return Whether the set did not already exist and was successfully registered
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands | Sets")
	UE_API bool RegisterCommandSet(FName SetName);

	/** Checks whether the given set is currently registered in the subsystem */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands | Sets")
	UE_API bool IsCommandSetRegistered(FName SetName) const;
	
	/**
	 * Unregisters the corresponding command set with all commands registered within it
	 * @return Whether the command set existed and was successfully unregistered
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands | Sets")
	UE_API bool UnregisterCommandSet(FName SetName);

	/** Enables or disables execution of commands registered within the given set */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands | Sets", Meta = (DisplayName = "Set Can Execute Commands"))
	UE_API void SetCanSetExecuteCommands(FName SetName, bool bShouldExecuteCommands);

	/** Checks whether commands in the given set can be executed. This will also check CanExecuteCommands at a global scope */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands | Sets", Meta = (DisplayName = "Can Execute Commands"))
	UE_API bool CanSetExecuteCommands(const FName SetName) const;

	
	/**
	 * Retrieves the list of names for all contexts currently registered in the subsystem.
	 * This does not check whether the contexts are bound to any UI Command List. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands | Contexts")
	UE_API TArray<FName> GetAvailableContexts() const;
	
	/**
	 * Checks whether the context with the given name is currently registered in the subsystem
	 * This does not check whether the context is bound to any UI Command List. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands | Contexts")
	UE_API bool IsContextRegistered(FName ContextName) const;
	
	/**
	 * Retrieves the number of UI Command Lists registered within this context through the subsystem.
	 * UI Command Lists are typically used to bind the list of commands associated with a single UI (i.e. a single viewport).
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands | Contexts")
	UE_API int GetBindingCountForContext(FName ContextName);

	/**
	 * Checks whether the given input chord is already mapped to a command in the given context.
	 * This includes commands not registered through the subsystem.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Commands | Contexts")
	UE_API bool IsInputChordMapped(FName ContextName, FInputChord InputChord) const;	
	
private:
	/** The internal method to dynamically registers an editor command */
	UE_API bool RegisterNewScriptingCommand(FScriptingCommandInfo CommandInfo, FExecuteCommand OnExecuteCommand,
	                               FCanExecuteAction OnCanExecuteAction, bool bOverrideExisting = false);

	/** Registers a context command in its command lists. This assumes the command is already contained in a set */
	UE_API bool RegisterScriptingCommand(const TSharedRef<FScriptingCommand> ScriptingCommand);

	/** Unregisters a context command from its command lists. This assumes the command is already contained in a set */
	UE_API bool UnregisterScriptingCommand(const TSharedRef<FScriptingCommand> ScriptingCommand);

	/** The delegate bound to registered commands. It will broadcast back to the given dynamic delegate */
	UFUNCTION()
	static UE_API void HandleExecuteAction(FExecuteCommand OnExecuteAction, const FScriptingCommandInfo CommandInfo);

	/**
	 * The delegate used to check whether a command can be executed.
	 * @return true if the given delegate returns true and the given set and all subsystem commands are enabled
	 */
	UFUNCTION()
	UE_API bool HandleCanExecuteAction(FCanExecuteCommand OnCanExecuteAction, const FScriptingCommandInfo CommandInfo) const;

	/**
	 * The default delegate to check if a command can be executed (when the user does not provide a specific delegate).
	 * @return true if the given set and all subsystem commands are enabled
	 */
	UFUNCTION()
	UE_API bool DefaultCanExecuteAction(FName SetName) const;

private:
	/** The list of commands contexts. Each context contains its own list of registered commands and UI command lists */
	TMap<FName, FScriptingCommandsContext> CommandsInContext;
	
	/** The list of command sets. Each set contains its own list of registered commands */
	TMap<FName, FScriptingCommandSet> CommandSets;

	/** Whether the commands for all the sets of the subsystem can be executed */
	bool bCanExecuteCommands = true;
};

#undef UE_API
