// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SlateScriptingCommands : ModuleRules
	{
		public SlateScriptingCommands(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Slate",
					"SlateCore",
					"Engine"
				});
			
			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"InputCore"
			});
		}
	}
}

